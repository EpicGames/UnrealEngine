// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateStreamHandle.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateStreamHandle)

FStateStreamHandle::FStateStreamHandle(const FStateStreamHandle& Other)
:	Owner(Other.Owner)
,	IsInternal(Other.IsInternal)
,	Id(Other.Id)
{
	if (Owner && !IsInternal)
	{
		Owner->Game_AddRef(Id);
	}
}

FStateStreamHandle::FStateStreamHandle(FStateStreamHandle&& Other)
{
	Owner = Other.Owner;
	IsInternal = Other.IsInternal;
	Id = Other.Id;
	Other.Owner = nullptr;
	Other.Id = 0;
}

FStateStreamHandle& FStateStreamHandle::operator=(FStateStreamHandle&& Other)
{
	if (Owner && !IsInternal)
	{
		Owner->Game_Release(Id);
	}
	Owner = Other.Owner;
	IsInternal = Other.IsInternal;
	Id = Other.Id;
	Other.Owner = nullptr;
	Other.Id = 0;
	return *this;
}

FStateStreamHandle& FStateStreamHandle::operator=(const FStateStreamHandle& Other)
{
	if (Owner == Other.Owner && Id == Other.Id)
	{
		return *this;
	}
	if (Owner && !IsInternal)
	{
		Owner->Game_Release(Id);
	}
	Owner = Other.Owner;
	IsInternal = Other.IsInternal;
	Id = Other.Id;
	if (Owner && !IsInternal)
		Owner->Game_AddRef(Id);
	return *this;
}

FStateStreamHandle::~FStateStreamHandle()
{
	if (Owner && !IsInternal)
	{
		Owner->Game_Release(Id);
	}
}

FStateStreamHandle::FStateStreamHandle(IStateStreamHandleOwner& O, uint32 I)
:	Owner(&O)
,	Id(I)
{
}

FStateStreamHandle::FStateStreamHandle(FStateStreamCopyContext& Context, const FStateStreamHandle& Other)
:	Owner(Other.Owner)
,	IsInternal(Context.IsInternal)
,	Id(Other.Id)
{
	if (Owner && !IsInternal)
	{
		Owner->Game_AddRef(Id);
	}
}

void FStateStreamHandle::Apply(FStateStreamCopyContext& Context, const FStateStreamHandle& Other)
{
	if (!IsInternal)
	{
		check(!Other.IsInternal); // If handle is not internal we should be on game side and the other must also not be internal
		if (Other.Owner)
		{
			Other.Owner->Game_AddRef(Other.Id);
		}
		if (Owner)
		{
			Owner->Game_Release(Id);
		}
	}

	Owner = Other.Owner;
	Id = Other.Id;
}

bool FStateStreamHandle::operator==(const FStateStreamHandle& Other) const
{
	return Id == Other.Id && Owner == Other.Owner;
}

void* FStateStreamHandle::Render_GetUserData() const
{
	if (Owner && Id)
	{
		check(IsInternal);
		return Owner->Render_GetUserData(Id);
	}
	return nullptr;
}

void FStateStreamHandle::MakeInternal()
{
	if(Owner && !IsInternal)
	{
		Owner->Game_Release(Id);
	}
	IsInternal = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
