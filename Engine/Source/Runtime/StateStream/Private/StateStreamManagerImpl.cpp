// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateStreamManagerImpl.h"
#include "StateStreamCreator.h"
#include "StateStreamDebugRenderer.h"

#define STATE_STREAM_DEBUG_CREATE_DESTROY 0

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FStateStreamManagerImpl::Game_CreateLane()
{
	return 0;
}
void FStateStreamManagerImpl::Game_SetLaneUserData(uint32 LaneId, void* UserData)
{
	for (const StateStreamRec& Rec : StateStreams)
	{
		Rec.Stream->Game_SetLaneUserData(LaneId, UserData);
	}
}

void FStateStreamManagerImpl::Game_DestroyLane(uint32 LaneId)
{
}

void FStateStreamManagerImpl::Game_BeginTick(uint32 LaneId)
{
	check(!bIsInTick);
	check (!bGameExited);
	bIsInTick = true;
	for (const StateStreamRec& Rec : StateStreams)
	{
		Rec.Stream->Game_BeginTick(LaneId);
	}
}

void FStateStreamManagerImpl::Game_EndTick(double AbsoluteTime, uint32 LaneId)
{
	check(bIsInTick);
	bIsInTick = false;
	for (const StateStreamRec& Rec : StateStreams)
	{
		Rec.Stream->Game_EndTick(AbsoluteTime, LaneId);
	}
}

void FStateStreamManagerImpl::Game_Exit()
{
	check(!bIsInTick);
	for (const StateStreamRec& Rec : StateStreams)
	{
		Rec.Stream->Game_Exit();
	}
	bGameExited = true;
}

bool FStateStreamManagerImpl::Game_IsInTick(uint32 LaneId)
{
	return bIsInTick;
}

uint32 FStateStreamManagerImpl::Game_SetDefaultLane(uint32 LaneId)
{
	uint32 OldLaneId = ActiveDefaultLaneId;
	ActiveDefaultLaneId = LaneId;
	for (const StateStreamRec& Rec : StateStreams)
	{
		Rec.Stream->Game_SetDefaultLane(LaneId);
	}
	return OldLaneId;
}

void* FStateStreamManagerImpl::Game_GetStreamPointer(uint32 Id)
{
	check(Id < uint32(StateStreamsLookup.Num()));
	return StateStreamsLookup[Id]->Game_GetVoidPointer();
}

void FStateStreamManagerImpl::Render_Register(IStateStream& Stream, bool TakeOwnership)
{
	StateStreamRec Rec { &Stream, TakeOwnership };
	StateStreams.Add(Rec);
	
	uint32 Id = Stream.GetId();
	if (uint32(StateStreamsLookup.Num()) <= Id)
	{
		StateStreamsLookup.SetNum(Id+1);
	}
	check(StateStreamsLookup[Id] == nullptr);
	StateStreamsLookup[Id] = &Stream;
}

void FStateStreamManagerImpl::Render_RegisterDependency(uint32 FromId, uint32 ToId)
{
	// TODO
}

void FStateStreamManagerImpl::Render_RegisterDependency(IStateStream& From, IStateStream& To)
{
	Render_RegisterDependency(From.GetId(), To.GetId());
}

void FStateStreamManagerImpl::Render_Update(double AbsoluteTime)
{
#if STATE_STREAM_DEBUG_CREATE_DESTROY
	static uint32 Counter = 0;
	++Counter;
	for (const StateStreamRec& Rec : StateStreams)
	{
		Rec.Stream->Render_Enable(((Counter/100) % 2) ? true : false);
	}
#endif

	for (const StateStreamRec& Rec : StateStreams)
	{
		Rec.Stream->Render_Update(AbsoluteTime);
	}

	for (const StateStreamRec& Rec : StateStreams)
	{
		Rec.Stream->Render_PostUpdate();
	}
}

void FStateStreamManagerImpl::Render_Exit()
{
	GarbageCollectTask.Wait();

	for (const StateStreamRec& Rec : StateStreams)
	{
		Rec.Stream->Render_Exit();
	}
	bRenderExited = true;
}

void FStateStreamManagerImpl::Render_GarbageCollect(bool AsTask)
{
	GarbageCollectTask.Wait();

	auto Func = [this]()
		{
			for (const StateStreamRec& Rec : StateStreams)
			{
				Rec.Stream->Render_GarbageCollect();
			}
		};
	if (AsTask)
	{
		GarbageCollectTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, MoveTemp(Func));
	}
	else
	{
		Func();
	}
}

IStateStream* FStateStreamManagerImpl::Render_GetStream(uint32 Id)
{
	check(Id < uint32(StateStreamsLookup.Num()));
	return StateStreamsLookup[Id];
}

void FStateStreamManagerImpl::Game_DebugRender(IStateStreamDebugRenderer& Renderer, uint32 LaneId)
{
	TStringBuilder<1024> DebugLine;
	
	// Display the number of state streams
	DebugLine.Reset();
	DebugLine.Appendf(TEXT("Num State streams = %d"), StateStreams.Num());
	Renderer.DrawText(*DebugLine, FColor::Blue);
	
	// Display per state stream informations
	for (const StateStreamRec& Rec : StateStreams)
	{
		Rec.Stream->Game_DebugRender(Renderer);
	}
}

FStateStreamManagerImpl::~FStateStreamManagerImpl()
{
	for (const StateStreamRec& Rec : StateStreams)
	{
		if (Rec.Owned)
		{
			delete Rec.Stream;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FStateStreamCreator* FStateStreamCreator::First;

FStateStreamCreator::FStateStreamCreator(uint32 Id_, FRegisterFunction&& InRegisterFunction, FUnregisterFunction&& InUnregisterFunction)
:	RegisterFunction(MoveTemp(InRegisterFunction))
,	UnregisterFunction(MoveTemp(InUnregisterFunction))
,	Id(Id_)
{
	Prev = nullptr;
	for (FStateStreamCreator* It=First; It; It=It->Next)
	{
		if (Id < It->Id)
			break;
		Prev = It;
	}

	if (!Prev)
	{
		Next = First;
		First = this;
	}
	else
	{
		Next = Prev->Next;
		Prev->Next = this;
	}

	if (Next)
	{
		Next->Prev = this;
	}
}

FStateStreamCreator::~FStateStreamCreator()
{
	if (Next)
	{
		Next->Prev = Prev;
	}

	if (Prev)
	{
		Prev->Next = Next;
	}
	else
	{
		First = Next;
	}

	Next = nullptr;
	Prev = nullptr;
}

void FStateStreamCreator::RegisterStateStreams(const FStateStreamRegisterContext& Context)
{
	for (FStateStreamCreator* It=First; It; It=It->Next)
	{
		It->RegisterFunction(Context);
	}
}

void FStateStreamCreator::UnregisterStateStreams(const FStateStreamUnregisterContext& Context)
{
	for (FStateStreamCreator* It=First; It; It=It->Next)
	{
		It->UnregisterFunction(Context);
	}
}

void FStateStreamRegisterContext::Register(IStateStream& StateStream, bool TakeOwnership) const
{
	Manager.Render_Register(StateStream, TakeOwnership);
}

void FStateStreamRegisterContext::RegisterDependency(uint32 FromId, uint32 ToId) const
{
	Manager.Render_RegisterDependency(FromId, ToId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
