// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransformStateStreamImpl.h"
#include "StateStreamCreator.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

FTransformObject::~FTransformObject()
{
	if (Parent)
	{
		Parent->RemoveListener(this);
		Parent = nullptr;
	}
	check(!First);
}

FTransformObject::Info FTransformObject::GetInfo()
{
	// TODO: Race condition safe
	if (bDirty)
	{
		WorldTransform = LocalState->GetLocalTransform();
		bVisible = LocalState->GetVisible();

		if (Parent)
		{
			Info ParentInfo = Parent->GetInfo();
			WorldTransform *= ParentInfo.WorldTransform;
			bVisible |= ParentInfo.bVisible;
		}

		bDirty = false;
	}

	return { WorldTransform, LocalState->GetBoneTransforms(), bVisible };
}

void FTransformObject::AddListener(FTransformObjectListener* Listener)
{
	if (!First)
	{
		First = Listener;
	}
	else
	{
		Listener->Next = First;
		First->Prev = Listener;
		First = Listener;
	}
}

void FTransformObject::RemoveListener(FTransformObjectListener* Listener)
{
	if (Listener->Prev)
	{
		Listener->Prev->Next = Listener->Next;
	}
	else
	{
		First = Listener->Next;
	}

	if (Listener->Next)
	{
		Listener->Next->Prev = Listener->Prev;
	}

	Listener->Next = nullptr;
	Listener->Prev = nullptr;
}

void FTransformObject::CallListeners()
{
	for (FTransformObjectListener* It = First; It; It = It->Next)
	{
		It->OnTransformObjectDirty();
	}
}

void FTransformObject::OnTransformObjectDirty()
{
	bDirty = true;
	CallListeners();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTransformStateStreamImpl::SetParent(FTransformObject* Object, const FTransformDynamicState& Ds)
{
	const FTransformHandle& ParentHandle = Ds.GetParent();
	if (!ParentHandle.IsValid() || Object->Parent)
	{
		return;
	}

	FTransformObject*& ParentUserData = Render_GetUserData(ParentHandle);
	FTransformObject* ParentObject = ParentUserData;
	if (!ParentObject)
	{
		ParentObject = new FTransformObject();
		ParentObject->AddRef();
		ParentUserData = ParentObject;

		SetParent(ParentObject, Render_GetDynamicState(ParentHandle));
	}

	ParentObject->AddListener(Object);
	Object->Parent = ParentObject;
}

void FTransformStateStreamImpl::Render_OnCreate(const FTransformStaticState& Ss, const FTransformDynamicState& Ds, FTransformObject*& UserData, bool IsDestroyedInSameFrame)
{
	FTransformObject* Object = UserData;
	if (!Object)
	{
		Object = new FTransformObject();
		Object->AddRef();
		Object->LocalState = &Ds;
		UserData = Object;
	}

	SetParent(Object, Ds);
}

void FTransformStateStreamImpl::Render_OnUpdate(const FTransformStaticState& Ss, const FTransformDynamicState& Ds, FTransformObject*& UserData)
{
		FTransformObject* Object = UserData;
		if (Ds.LocalTransformModified() || Ds.VisibleModified() || Ds.BoneTransformsModified())
		{
			Object->OnTransformObjectDirty();
		}

		if (Ds.ParentModified())
		{
			SetParent(Object, Ds);
		}
}

void FTransformStateStreamImpl::Render_OnDestroy(const FTransformStaticState& Ss, const FTransformDynamicState& Ds, FTransformObject*& UserData)
{
	UserData->Release();
	UserData = nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FStateStreamCreator TransformStateStreamCreator(TransformStateStreamId,
	[](const FStateStreamRegisterContext& Context) { Context.Register(*new FTransformStateStreamImpl(), true); },
	[](const FStateStreamUnregisterContext& Context) {});

////////////////////////////////////////////////////////////////////////////////////////////////////
