// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "Misc/Optional.h"
#include "Misc/ScopeRWLock.h"
#include "StateStream.h"
#include "StateStreamDebugRenderer.h"
#include "StateStreamDefinitions.h"
#include "StateStreamStore.h"
#include "ProfilingDebugging/CountersTrace.h"

// State stream stat group
DECLARE_STATS_GROUP(TEXT("StateStream"), STATGROUP_StateStream, STATCAT_Advanced);

// Declare state stream counter to the stat system and to unreal insights 
#define DECLARE_STATESTREAM_COUNTER_STATS(Name, Group) \
	DECLARE_DWORD_COUNTER_STAT(TEXT(#Group) TEXT("|") TEXT(#Name), STAT_StateStream_Counters_##Group##_##Name, STATGROUP_StateStream) \
	TRACE_DECLARE_INT_COUNTER(StateStream_Counters_##Group##_##Name, TEXT("StateStream/") TEXT(#Group) TEXT("/") TEXT(#Name));

// Define state stream counter to the stat system and to unreal insights 
#define DEFINE_STATESTREAM_COUNTER_STATS(Name, Group, Value) \
	SET_DWORD_STAT(STAT_StateStream_Counters_##Group##_##Name, Value); \
	TRACE_COUNTER_SET(StateStream_Counters_##Group##_##Name, Value)

////////////////////////////////////////////////////////////////////////////////////////////////////
// Settings struct - Can be inherited to set settings on state stream. It mainly contains all the data types used 
// by state stream (static state / dynamic state / handle type / interface type) + id and debug name

template<typename TInterfaceType, typename TInstanceUserDataType = void, typename TLaneUserDataType = void>
struct TStateStreamSettings
{
	using InterfaceType = TInterfaceType;
	using InstanceUserDataType = TInstanceUserDataType;
	using LaneUserDataType = TLaneUserDataType;
	enum { Id = InterfaceType::Id };
	static inline constexpr const TCHAR* DebugName = InterfaceType::Handle::DebugName;
	static inline constexpr bool SkipCreatingDeletes = false;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// TStateStream is a generic implementation of IStateStream that contains all the boiler plate code
// related to ticks, interpolation, etc. Should be the default goto implementation.
// Inherit this class for each type of state stream
// Provide a subclass of TStateStreamSettings as template parameter

template<typename Settings>
class TStateStream : public IStateStream, public Settings::InterfaceType, public IStateStreamHandleOwner
{
public:
	using InterfaceType = typename Settings::InterfaceType;
	using FHandle = typename InterfaceType::Handle;
	using FStaticState = typename InterfaceType::StaticState;
	using FDynamicState = typename InterfaceType::DynamicState;
	using FInstanceUserDataType = typename Settings::InstanceUserDataType;
	using FLaneUserDataType = typename Settings::LaneUserDataType;
	enum { Id = Settings::Id };

	//~ Begin InterfaceType interface
	virtual FHandle Game_CreateInstance(const FStaticState& Ss, const FDynamicState& Ds, uint32 LaneId = DefaultLaneId) override final;
	//~ End InterfaceType interface

	//~ Begin IStateStreamHandleOwner interface (used by state stream handles on game side)
	virtual void Game_AddRef(uint32 HandleId) override final;
	virtual void Game_Release(uint32 HandleId) override final;
	virtual void Game_Update(uint32 HandleId, const void* Ds, double TimeFactor, uint64 UserData) override;
	virtual void* Game_Edit(uint32 HandleId, double TimeFactor, uint64 UserData) override;
	virtual void* Render_GetUserData(uint32 HandleId) override final;
	//~ End IStateStreamHandleOwner interface
	
	//~ Begin IStateStream interface (used by StateStreamManagerImpl)
	virtual void Game_CreateLane() override {}
	virtual void Game_SetLaneUserData(uint32 LaneId, void* UserData) override { LaneUserData = (FLaneUserDataType*)UserData; }
	virtual void Game_DestroyLane(uint32 LaneId) override {}
	virtual void Game_BeginTick(uint32 LaneId) override;
	virtual void Game_EndTick(StateStreamTime AbsoluteTime, uint32 LaneId) override;
	virtual void Game_SetDefaultLane(uint32 LaneId) override;
	virtual void Game_Exit() override;
	virtual void* Game_GetVoidPointer() override;
	virtual void Game_DebugRender(IStateStreamDebugRenderer& Renderer) override;
	virtual void Render_Update(StateStreamTime AbsoluteTime) override;
	virtual void Render_PostUpdate() override;
	virtual void Render_Exit() override;
	virtual void Render_GarbageCollect() override;
	virtual void Render_Enable(bool Enable) override;
	virtual const TCHAR* GetDebugName() override;
	virtual uint32 GetId() override;
	//~ End IStateStream interface

	// Get the instance user data on the render side
	FInstanceUserDataType*& Render_GetUserData(const FHandle& Handle);
	
	// Get the instance interpolated dynamic state on the render side
	const FDynamicState& Render_GetDynamicState(const FHandle& Handle);
	
	// Specialize to do custom things when state has been created / updated or destroyed
	virtual void Render_OnCreate(const FStaticState& Ss, const FDynamicState& Ds, FInstanceUserDataType*& UserData, bool IsDestroyedInSameFrame) {}
	virtual void Render_OnUpdate(const FStaticState& Ss, const FDynamicState& Ds, FInstanceUserDataType*& UserData) {}
	virtual void Render_OnDestroy(const FStaticState& Ss, const FDynamicState& Ds, FInstanceUserDataType*& UserData) {}

	// ... or use template specialization to avoid virtual calls
	void Render_OnCreateInline(const FStaticState& Ss, const FDynamicState& Ds, FInstanceUserDataType*& UserData, bool IsDestroyedInSameFrame) { Render_OnCreate(Ss, Ds, UserData, IsDestroyedInSameFrame); }
	void Render_OnUpdateInline(const FStaticState& Ss, const FDynamicState& Ds, FInstanceUserDataType*& UserData) { Render_OnUpdate(Ss, Ds, UserData); }
	void Render_OnDestroyInline(const FStaticState& Ss, const FDynamicState& Ds, FInstanceUserDataType*& UserData) { Render_OnDestroy(Ss, Ds, UserData); }
	
	// Get the used instance counts within the streams
	uint32 GetUsedInstancesCount() const { return Instances.GetUsedCount(); }
	
	// Get the used dynamic states within the streams
	uint32 GetUsedDynamicStatesCount() const { return DynamicStates.GetUsedCount(); }

	TStateStream() = default;
	~TStateStream() = default;

protected:
	
	// Information about instance.
	struct FInstance
	{
		FInstance() = default;
		FInstance(const FStaticState& Ss, uint32 Rc, uint32 Ct) : StaticState(Ss), RefCount(Rc), CreateTick(Ct) {}
		FStaticState StaticState;
		uint32 RefCount = 0;
		uint32 CreateTick = 0;
		uint32 DeleteTick = ~0u;
		TOptional<FDynamicState> RendDynamicState;
		FInstanceUserDataType* UserData = nullptr;
	};
	
	// Tick produced by game side.
	struct FTick
	{
		TArray<uint32> DynamicStates; // Contains for each instance the index of the dynamic states used by this tick (some might have been destroyed but index not reused)
		TBitArray<> ModifiedInstances; // Contains bits saying which of the instances that have been modified in this tick.
		FTick* PrevTick = nullptr; // Tick with earlier time
		FTick* NextTick = nullptr; // Tick with newer time
		StateStreamTime Time = 0; // Time Tick finished
		uint32 Index = 0; // Index of tick (created by TickCounter)
	};
	
	// Get the dynamic state to be edited on GT in between begin/end tick
	FDynamicState& Edit(uint32 HandleId, double TimeFactor);

	// Update the editable dynamic state
	template<typename InDynamicState>
	void Update(uint32 HandleId, const InDynamicState& Ds, double TimeFactor);

	// Interpolate and set the render dynamic states + call create / update / destroy callbacks. Called from the Render_Update
	void ApplyChanges(const FTick& Tick, StateStreamTime Time, uint32 PrevTickIndex, StateStreamTime PrevTime, const TBitArray<>& ModifiedInstances);

	// Make the state internal in case it has been released by its owner
	template<typename T>
	void MakeInternal(T& State);

	uint32 ActiveDefaultLaneId = DefaultLaneId;

	FLaneUserDataType* LaneUserData = nullptr;

	// Lock in order to write into the current tick
	FRWLock CurrentTickLock;
	
	// Tick being worked on in game
	FTick* CurrentTick = nullptr; 
	
	// Oldest finished tick available to rendering
	FTick* OldestAvailableTick = nullptr; 
	
	// Newest finished tick available to rendering
	FTick* NewestAvailableTick = nullptr; 

	// Instances registered in the state stream 
	TStateStreamStore<FInstance> Instances;
	
	// Dynamic states that will be referenced in the different ticks 
	TStateStreamStore<FDynamicState> DynamicStates; // Store for dynamic states.

	// Tick counter used to increment tick indices
	uint32 TickCounter = 1;

	// Last used tick for rendering.
	FTick* RendTick = nullptr; 
	
	// Last used time for rendering
	StateStreamTime RendTime = 0;
	
	// Boolean to trigger the render update 
	bool RendEnabledRequested = true;
	
	// Check if the render update has been enabled or not
	bool RendEnabled = true;

	// List of instances to be destroyed
	TArray<FInstance*> DeferredDestroys;

	TStateStream(const TStateStream&) = delete;
	TStateStream& operator=(const TStateStream&) = delete;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename Settings>
typename TStateStream<Settings>::FHandle TStateStream<Settings>::Game_CreateInstance(const FStaticState& Ss, const FDynamicState& Ds, uint32 LaneId)
{
	if (LaneId == DefaultLaneId)
	{
		check(ActiveDefaultLaneId != DefaultLaneId);
		LaneId = ActiveDefaultLaneId;
	}

	check(CurrentTick);

	// We usually have several dynamic states stored for each instance
	// and consecutive  ticks could refer the same dynamic state if nothing changed
	const uint32 InstanceIndex = Instances.Emplace(Ss, 1, CurrentTick->Index);
	const uint32 DynamicStateIndex = DynamicStates.Add(Ds);

	CurrentTickLock.WriteLock();

	// We make sure the current tick dynamic states / instances have at least the stream instances size
	if (static_cast<uint32>(CurrentTick->DynamicStates.Num()) <= InstanceIndex)
	{
		CurrentTick->DynamicStates.SetNum(InstanceIndex + 1);
		CurrentTick->ModifiedInstances.SetNum(InstanceIndex + 1, false);
	}
	CurrentTick->ModifiedInstances[InstanceIndex] = true;
	CurrentTick->DynamicStates[InstanceIndex] = DynamicStateIndex;

	CurrentTickLock.WriteUnlock();

	return { *this, InstanceIndex + 1 };
}

template<typename Settings>
void TStateStream<Settings>::Game_AddRef(uint32 HandleId)
{
	check(CurrentTick);
	check(HandleId != 0);
	uint32 InstanceIndex = HandleId - 1;
	FInstance& Instance = Instances[InstanceIndex];
	check(Instance.RefCount);
	++Instance.RefCount;
}

template<typename Settings>
void TStateStream<Settings>::Game_Release(uint32 HandleId)
{
	check(CurrentTick);
	check(HandleId != 0);
	uint32 InstanceIndex = HandleId - 1;
	FInstance& Instance = Instances[InstanceIndex];
	check(Instance.RefCount);
	if (--Instance.RefCount)
	{
		return;
	}

	// If no more refs we are going to delete the tick store its index onto the delete tick
	// the static and dynamic states should be made internal 
	Instance.DeleteTick = CurrentTick->Index;
	MakeInternal(Instance.StaticState);

	CurrentTickLock.ReadLock();

	CurrentTick->ModifiedInstances[InstanceIndex] = true;
	uint32 DsIndex = CurrentTick->DynamicStates[InstanceIndex];

	CurrentTickLock.ReadUnlock();

	MakeInternal(DynamicStates[DsIndex]);
}

template<typename Settings>
void TStateStream<Settings>::Game_Update(uint32 HandleId, const void* Ds, double TimeFactor, uint64 UserData)
{
	FDynamicState& DsState = Edit(HandleId, TimeFactor);
	FStateStreamCopyContext Context;
	Context.IsInternal = 0;
	DsState.Apply(Context, *static_cast<const FDynamicState*>(Ds));
}

template<typename Settings>
void* TStateStream<Settings>::Game_Edit(uint32 HandleId, double TimeFactor, uint64 UserData)
{
	return &Edit(HandleId, TimeFactor);
}

template<typename Settings>
typename TStateStream<Settings>::FDynamicState& TStateStream<Settings>::Edit(uint32 HandleId, double TimeFactor)
{
	check(HandleId != 0);
	uint32 InstanceIndex = HandleId - 1;

	check(CurrentTick);
	check(InstanceIndex < uint32(CurrentTick->DynamicStates.Num()));

	CurrentTickLock.ReadLock();

	FDynamicState* DsState;

	if (CurrentTick->ModifiedInstances[InstanceIndex])
	{
		// If the instance had already been modified just return the 
		// current tick dynamic states for that instance
		uint32 DsIndex = CurrentTick->DynamicStates[InstanceIndex];
		CurrentTickLock.ReadUnlock();
		DsState = &DynamicStates[DsIndex];
	}
	else
	{
		// If not modified, create a new dynamic state and reference 
		// it in the current tick ones. 
		uint32& IndexRef = CurrentTick->DynamicStates[InstanceIndex];
		uint32 OldIndex = IndexRef;
		CurrentTick->ModifiedInstances[InstanceIndex] = true;
		uint32 DsIndex;
		void* DsPtr = DynamicStates.AddUninitialized(DsIndex);
		IndexRef = DsIndex;
		CurrentTickLock.ReadUnlock();

		FDynamicState& OldDs = DynamicStates[OldIndex];
		DsState = new (DsPtr) FDynamicState(OldDs);
		
		// Why do wee need to make the old state internal since 
		// this state could still be referenced by previous ticks?
		MakeInternal(OldDs);
	}
	return *DsState;
}

template<typename Settings>
template<typename InDynamicState>
void TStateStream<Settings>::Update(uint32 HandleId, const InDynamicState& Ds, double TimeFactor)
{
	/*
	check(HandleId != 0);
	uint32 InstanceIndex = HandleId - 1;

	check(CurrentTick);
	check(InstanceIndex < uint32(CurrentTick->DynamicStates.Num()));

	CurrentTickLock.ReadLock();
	uint32 DsIndex = CurrentTick->DynamicStates[InstanceIndex];
	auto& CurrentState = *(decltype(InDynamicState)*)DynamicStates[DsIndex];
	if (!CurrentState.Diff(InDs)) // TODO
	{
		return;
	}
	*/
	FStateStreamCopyContext Context;
	Context.IsInternal = 0;
	
	// Get the editable state and apply the input dynamic state
	InDynamicState& DsState = Edit(HandleId, TimeFactor);
	DsState.Apply(Context, Ds);
}

template<typename Settings>
void* TStateStream<Settings>::Render_GetUserData(uint32 HandleId)
{
	check(HandleId);
	uint32 InstanceIndex = HandleId - 1;
	return Instances[InstanceIndex].UserData;
}

template<typename Settings>
void TStateStream<Settings>::Game_BeginTick(uint32 LaneId)
{
	check(!CurrentTick);
	
	// The new current tick for that stream is being created
	CurrentTick = new FTick();
	CurrentTick->Index = TickCounter++;
	if (NewestAvailableTick)
	{
		// We copy bu default the list of dynamic states referenced by the last available tick 
		// in case no instances will change and set the modified flag to false
		CurrentTick->DynamicStates = NewestAvailableTick->DynamicStates;
		CurrentTick->ModifiedInstances.SetNum(CurrentTick->DynamicStates.Num(), false);
	}
}

template<typename Settings>
void TStateStream<Settings>::Game_EndTick(StateStreamTime AbsoluteTime, uint32 LaneId)
{
	check(CurrentTick);

	CurrentTick->Time = AbsoluteTime;
	CurrentTick->PrevTick = NewestAvailableTick;
	if (NewestAvailableTick)
	{
		check(NewestAvailableTick->Time <= AbsoluteTime);
		NewestAvailableTick->NextTick = CurrentTick;
	}
	else
	{
		OldestAvailableTick = CurrentTick;
	}
		
	// the current tick is being reset and the newest one available to rendering 
	// is getting updated
	NewestAvailableTick = CurrentTick;
	CurrentTick = nullptr;
}

template<typename Settings>
void TStateStream<Settings>::Game_SetDefaultLane(uint32 LaneId)
{
	ActiveDefaultLaneId = LaneId;
}

template<typename Settings>
void TStateStream<Settings>::Game_Exit()
{
	check(!CurrentTick);
	/*
	for (TConstSetBitIterator<> It(NewestAvailableTick->Modif); It; ++It)
	{

	}
	*/
}

template<typename Settings>
void* TStateStream<Settings>::Game_GetVoidPointer()
{
	return static_cast<InterfaceType*>(this);
}

template<typename Settings>
uint32 TStateStream<Settings>::GetId()
{
	return Id;
}

template<typename Settings>
void TStateStream<Settings>::Render_Update(StateStreamTime AbsoluteTime)
{
	bool Requested = RendEnabledRequested;
	if (Requested != RendEnabled)
	{
		RendEnabled = Requested;
		
		// If the render update is enable and if it has changed we are in creation mode
		// We need to find the tick with previous time lower than the requested absolute time
		// and use it to fill the render dynamic state
		if (RendEnabled)
		{
			// Figure out which tick to use and recreate instances
			check(NewestAvailableTick);
			FTick* Tick = NewestAvailableTick;
			while (true)
			{
				if (AbsoluteTime >= Tick->Time || !Tick->PrevTick)
				{
					break;
				}
				Tick = Tick->PrevTick;
			}

			RendTick = Tick;
			RendTime = AbsoluteTime;

			for (uint32 InstanceIndex = 0, E = RendTick->DynamicStates.Num(); InstanceIndex != E; ++InstanceIndex)
			{
				FInstance& Instance = Instances[InstanceIndex];
				if (Instance.CreateTick > Tick->Index || Instance.DeleteTick <= Tick->Index)
				{
					continue;
				}
				check(!Instance.RendDynamicState.IsSet());
				uint32 DynamicStateIndex = Tick->DynamicStates[InstanceIndex];
				FStateStreamCopyContext Context;
				
				// Why do we use the render dynamic state from the prev tick and not interpolate from prev->next?
				Instance.RendDynamicState.Emplace(Context, DynamicStates[DynamicStateIndex]);
				Render_OnCreateInline(Instance.StaticState, *Instance.RendDynamicState, Instance.UserData, false);
			}
		}
		// If the render update is disable and if it has changed we are in deletion mode
		// We need to to reset all th instances render dynamic state
		else if (RendTick)
		{
			// Cleanup all instances
			for (uint32 InstanceIndex = 0, E = RendTick->DynamicStates.Num(); InstanceIndex != E; ++InstanceIndex)
			{
				FInstance& Instance = Instances[InstanceIndex];
				if (!Instance.RendDynamicState)
				{
					continue;
				}
				Render_OnDestroyInline(Instance.StaticState, *Instance.RendDynamicState, Instance.UserData);
				Instance.RendDynamicState.Reset();
				checkf(!Instance.UserData, TEXT("Make sure to clear UserData in Render_OnDestroy"));
			}
			RendTick = nullptr;
		}
		return;
	}
	else if (!RendEnabled)
	{
		return;
	}

	if (!NewestAvailableTick || AbsoluteTime == RendTime)
	{
		return;
	}

	check(AbsoluteTime > RendTime); // Only play forward for now

	uint32 PrevTick = 0;
	bool IsFirstTick = false;
	if (RendTick)
	{
		PrevTick = RendTick->Index;
	}
	else
	{
		RendTick = OldestAvailableTick;
		IsFirstTick = true;
	}

	StateStreamTime PrevTime = RendTime;
	RendTime = AbsoluteTime;

	// We are still inside the same tick
	if (RendTime <= RendTick->Time)
	{
		// Just interpolate or apply RendTick
		ApplyChanges(*RendTick, RendTime, PrevTick, PrevTime, RendTick->ModifiedInstances);
		return;
	}
		
	// We were at exact end of last handled tick.. move into next
	if (PrevTime == RendTick->Time)
	{
		FTick* NewRendTick = RendTick->NextTick;

		if (!NewRendTick) // We've caught up with game.. set RendTime back to PrevTime and return
		{
			RendTime = PrevTime;
			return;
		}

		if (!IsFirstTick)
			RendTick = NewRendTick;

		// We don't need to include RendTick Modifications.
		if (RendTime <= RendTick->Time)
		{
			// Just interpolate or apply RendTick
			ApplyChanges(*RendTick, RendTime, PrevTick, PrevTime, RendTick->ModifiedInstances);
			return;
		}
	}

	// We are overlapping between two or more ticks
	TBitArray<> ModifiedInstances(RendTick->ModifiedInstances);
	while (RendTick->Time < RendTime)
	{
		if (!RendTick->NextTick)
		{
			RendTime = RendTick->Time;
			break;
		}
		RendTick = RendTick->NextTick;
		ModifiedInstances.CombineWithBitwiseOR(RendTick->ModifiedInstances, EBitwiseOperatorFlags::MaxSize);
	}

	ApplyChanges(*RendTick, RendTime, PrevTick, PrevTime, ModifiedInstances);
}

template<typename Settings>
void TStateStream<Settings>::Render_PostUpdate()
{
	for (FInstance* Instance : DeferredDestroys)
	{
		Render_OnDestroyInline(Instance->StaticState, *Instance->RendDynamicState, Instance->UserData);
		Instance->RendDynamicState.Reset();
		checkf(!Instance->UserData, TEXT("Make sure to clear UserData in Render_OnDestroy"));
	}
	DeferredDestroys.SetNum(0);
}

template<typename Settings>
void TStateStream<Settings>::Render_Exit()
{
	if (NewestAvailableTick)
	{
		StateStreamTime maxTime = (StateStreamTime)std::numeric_limits<StateStreamTime>::max();
		Render_Update(maxTime);
		//Render_GarbageCollect();
	}
}

template<typename Settings>
void TStateStream<Settings>::Render_GarbageCollect()
{
	if (!OldestAvailableTick)
	{
		return;
	}

	FTick* RendTickUsed = RendTick;
	if (RendTick && RendTick->Time != RendTime && RendTick->PrevTick)
		RendTickUsed = RendTick->PrevTick;
	
	if (!RendTickUsed)
	{
		RendTickUsed = OldestAvailableTick->NextTick;
		if (!RendTickUsed)
		{
			return;
		}
	}

	FTick* Tick = OldestAvailableTick;
	while (Tick != RendTickUsed)
	{
		FTick* Next = Tick->NextTick;
		check (Next);

		// We can remove all DynamicStates that are different in Next tick since we know this is the last tick using the state
		for (TConstSetBitIterator<> It(Next->ModifiedInstances); It; ++It)
		{
			uint32 InstanceIndex = It.GetIndex();
			if (InstanceIndex >= uint32(Tick->DynamicStates.Num()))
			{
				continue;
			}

			// If instance was created in next tick we ignore this. If instance is deleted in tick we handle it further down
			FInstance& Instance = Instances[InstanceIndex];
			if (Instance.DeleteTick == Tick->Index || Instance.CreateTick == Next->Index)
			{
				continue;
			}

			uint32 DynamicStateIndex = Tick->DynamicStates[InstanceIndex];
			if (DynamicStateIndex == Next->DynamicStates[InstanceIndex])
			{
				continue;
			}

			DynamicStates.Remove(DynamicStateIndex);
		}

		// Remove all instances who was deleted in Tick
		for (TConstSetBitIterator<> It(Tick->ModifiedInstances); It; ++It)
		{
			uint32 InstanceIndex = It.GetIndex();
			FInstance& Instance = Instances[InstanceIndex];

			if (Instance.DeleteTick != Tick->Index)
			{
				continue;
			}

			DynamicStates.Remove(Tick->DynamicStates[InstanceIndex]);
			Instance.RendDynamicState.Reset();
			Instances.Remove(InstanceIndex);
		}

		delete Tick;
		Tick = Next;
	}
	OldestAvailableTick = Tick;

}

template<typename Settings>
void TStateStream<Settings>::Render_Enable(bool Enable)
{
	RendEnabledRequested = Enable;
}

template<typename Settings>
const TCHAR* TStateStream<Settings>::GetDebugName()
{
	return Settings::DebugName;
}

template<typename Settings>
void TStateStream<Settings>::Game_DebugRender(IStateStreamDebugRenderer& Renderer)
{
	TStringBuilder<1024> DebugLine;
	uint32 ModifiedCount = 0;
	if (FTick* Tick = NewestAvailableTick)
	{
		ModifiedCount = Tick->ModifiedInstances.CountSetBits();
	}
	DebugLine.Appendf(TEXT("State stream : [%s | %u/%u modified]"), GetDebugName(), ModifiedCount, Instances.GetUsedCount());
	Renderer.DrawText(*DebugLine, FColor::Orange);
}

template<typename Settings>
typename TStateStream<Settings>::FInstanceUserDataType*& TStateStream<Settings>::Render_GetUserData(const FHandle& Handle)
{
	uint32 Id = Handle.GetId();
	check(Id);
	uint32 InstanceIndex = Id - 1;
	return Instances[InstanceIndex].UserData;
}

template<typename Settings>
const typename TStateStream<Settings>::FDynamicState& TStateStream<Settings>::Render_GetDynamicState(const FHandle& Handle)
{
	uint32 Id = Handle.GetId();
	check(Id);
	uint32 InstanceIndex = Id - 1;
	return *Instances[InstanceIndex].RendDynamicState;
}

template<typename Settings>
void TStateStream<Settings>::ApplyChanges(const FTick& Tick, StateStreamTime Time, uint32 PrevTickIndex, StateStreamTime PrevTime, const TBitArray<>& ModifiedInstances)
{
	// TODO: This will do lots of allocations
	struct FCreateInfo { FInstance* Instance; bool IsDestroyInSameFrame; };
	TArray<FCreateInfo> Creates;
	TArray<FInstance*> Updates;
	TArray<FInstance*> Destroys;

	for (TConstSetBitIterator<> It(ModifiedInstances); It; ++It)
	{
		uint32 InstanceIndex = It.GetIndex();
		FInstance& Instance = Instances[InstanceIndex];

		if (Instance.DeleteTick <= PrevTickIndex) // Already deleted
		{
			continue;
		}

		FStaticState& Ss = Instance.StaticState;

		bool IsCreate = Instance.CreateTick > PrevTickIndex && Instance.CreateTick <= Tick.Index;
		bool IsDestroy = Instance.DeleteTick > PrevTickIndex && Instance.DeleteTick <= Tick.Index;

		if (IsCreate)
		{
			if (Settings::SkipCreatingDeletes && IsDestroy)
			{
				continue;
			}

			check(!Instance.RendDynamicState.IsSet());
			uint32 DynamicStateIndex = Tick.DynamicStates[InstanceIndex];
			FStateStreamCopyContext Context;
			Instance.RendDynamicState.Emplace(Context, DynamicStates[DynamicStateIndex]);
		}
		else if (Instance.CreateTick == Tick.Index) // Still in the tick it was created, no interpolation possible
		{
			continue;
		}

		FDynamicState& Ds = *Instance.RendDynamicState;

		if (Tick.Time == Time || !Tick.PrevTick)
		{
			if (!IsCreate)
			{
				FStateStreamCopyContext Context;
				Ds.Apply(Context, DynamicStates[Tick.DynamicStates[InstanceIndex]]);
			}
		}
		else
		{
			const FTick& PrevTick = *Tick.PrevTick;
			uint32 FromIndex = PrevTick.DynamicStates[InstanceIndex];
			uint32 ToIndex = Tick.DynamicStates[InstanceIndex];
			
			StateStreamTime DeltaTime = Tick.Time - PrevTick.Time;
			StateStreamTime TimeInTo = Time - PrevTick.Time;
			double Factor = (DeltaTime == 0.0) ? 0.0f : double(TimeInTo) / DeltaTime;

			FDynamicState& From = DynamicStates[FromIndex];
			FDynamicState& To = DynamicStates[ToIndex];

			FStateStreamInterpolateContext Context;
			Context.Factor = Factor;
			Ds.Interpolate(Context, From, To);
		}

		if (IsCreate)
		{
			Creates.Add({&Instance, IsDestroy});
		}

		if (IsDestroy)
		{
			if (IsCreate)
			{
				DeferredDestroys.Add(&Instance);
			}
			else
			{
				Destroys.Add(&Instance);
			}
			continue;
		}

		if (!IsCreate)
		{
			Updates.Add(&Instance);
		}
	}

	for (FCreateInfo& Info : Creates)
	{
		FInstance& Instance = *Info.Instance;
		Render_OnCreateInline(Instance.StaticState, *Instance.RendDynamicState, Instance.UserData, Info.IsDestroyInSameFrame);
	}

	for (FInstance* Instance : Updates)
	{
		Render_OnUpdateInline(Instance->StaticState, *Instance->RendDynamicState, Instance->UserData);
	}

	for (FInstance* Instance : Destroys)
	{
		Render_OnDestroyInline(Instance->StaticState, *Instance->RendDynamicState, Instance->UserData);
		Instance->RendDynamicState.Reset();
		checkf(!Instance->UserData, TEXT("Make sure to clear UserData in Render_OnDestroy"));
	}
}

template<typename Settings>
template<typename T>
void TStateStream<Settings>::MakeInternal(T& State)
{
	FStateStreamHandle* Deps[256];
	for (uint32 I=0, E=State.GetDependencies(Deps, UE_ARRAY_COUNT(Deps)); I!=E; ++I)
	{
		Deps[I]->MakeInternal();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
