// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"


////////////////////////////////////////////////////////////////////////////////////////////////////
// FPow2ChunkedArray
//
// A chunk based array where each chunk is twice as large as the previous one.
// Adding and accessing elements are O(1)
// Add is threadsafe and keeping references is safe since it never reallocates

template<typename T, uint32 MinSize = 16, uint32 MaxSize = 16777216>
class FPow2ChunkedArray
{
public:
	enum : uint32 { SkipCount = FGenericPlatformMath::CeilLogTwo(MinSize) };
	enum : uint32 { BucketCount = FGenericPlatformMath::CeilLogTwo(MaxSize) - SkipCount + 1 };

	inline FPow2ChunkedArray();
	inline ~FPow2ChunkedArray();

	inline T& Add(const T& Value, uint32& OutIndex);
	inline T& Add(const T& Value);
	inline T& Add(T&& Value);

	inline const T& operator[](uint32 Index) const;
	inline T& operator[](uint32 Index);

	inline uint32 Num() const;

	inline const T& GetElementAt(uint32 Index) const;

	inline void* AddUninitialized(uint32& OutIndex);

	inline uint32 GetBucketIndex(uint32 Index) const;
	inline uint32 GetBucketStart(uint32 BucketIndex) const;
	inline uint32 GetBucketSize(uint32 BucketIndex) const;

private:
	std::atomic<uint32> Size;
	volatile int64 Buckets[BucketCount];
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Implementation

template<typename T, uint32 MinSize, uint32 MaxSize>
FPow2ChunkedArray<T, MinSize, MaxSize>::FPow2ChunkedArray()
{
	FMemory::Memset((int64*)Buckets, 0, sizeof(Buckets));
}

template<typename T, uint32 MinSize, uint32 MaxSize>
FPow2ChunkedArray<T, MinSize, MaxSize>::~FPow2ChunkedArray()
{
	uint32 Left = Size;
	for (uint32 BucketIndex=0; Left; ++BucketIndex)
	{
		T* Elements = reinterpret_cast<T*>(Buckets[BucketIndex]);
		uint32 LeftInBucket = FPlatformMath::Min(Left, GetBucketSize(BucketIndex));
		if (!std::is_trivially_destructible_v<T>)
		{
			for (uint32 I=0, E=LeftInBucket; I!=E; ++I)
			{
				Elements[I].~T();
			}
		}
		Left -= LeftInBucket;
		FMemory::Free(Elements);
	}
}

template<typename T, uint32 MinSize, uint32 MaxSize>
T& FPow2ChunkedArray<T, MinSize, MaxSize>::Add(const T& Value, uint32& OutIndex)
{
	return *new (AddUninitialized(OutIndex)) T(Value);
}

template<typename T, uint32 MinSize, uint32 MaxSize>
T& FPow2ChunkedArray<T, MinSize, MaxSize>::Add(const T& Value)
{
	uint32 OutIndex;
	return *new (AddUninitialized(OutIndex)) T(Value);
}

template<typename T, uint32 MinSize, uint32 MaxSize>
T& FPow2ChunkedArray<T, MinSize, MaxSize>::Add(T&& Value)
{
	uint32 OutIndex;
	return *new (AddUninitialized(OutIndex)) T(MoveTemp(Value));
}

template<typename T, uint32 MinSize, uint32 MaxSize>
const T& FPow2ChunkedArray<T, MinSize, MaxSize>::operator[](uint32 Index) const
{
	return GetElementAt(Index);
}

template<typename T, uint32 MinSize, uint32 MaxSize>
T& FPow2ChunkedArray<T, MinSize, MaxSize>::operator[](uint32 Index)
{
	return const_cast<T&>(GetElementAt(Index));
}

template<typename T, uint32 MinSize, uint32 MaxSize>
uint32 FPow2ChunkedArray<T, MinSize, MaxSize>::Num() const
{
	return Size;
}

template<typename T, uint32 MinSize, uint32 MaxSize>
const T& FPow2ChunkedArray<T, MinSize, MaxSize>::GetElementAt(uint32 Index) const
{
	check(Index < Size);
	uint32 BucketIndex = GetBucketIndex(Index);
	uint32 BucketStart = GetBucketStart(BucketIndex);
	uint32 BucketOffset = Index - BucketStart;
	return reinterpret_cast<const T*>(Buckets[BucketIndex])[BucketOffset];
}

template<typename T, uint32 MinSize, uint32 MaxSize>
void* FPow2ChunkedArray<T, MinSize, MaxSize>::AddUninitialized(uint32& OutIndex)
{
	uint32 Index = Size++;
	uint32 BucketIndex = GetBucketIndex(Index);
		
	volatile int64& BucketRef = Buckets[BucketIndex];
		
	int64 BucketRes = FPlatformAtomics::AtomicRead(&BucketRef);
	if (!BucketRes)
	{
		void* NewPtr = FMemory::Malloc(GetBucketSize(BucketIndex) * sizeof(T), alignof(T));
		BucketRes = FPlatformAtomics::InterlockedCompareExchange(&BucketRef, int64(NewPtr), 0);
		if (BucketRes)
		{
			FMemory::Free(NewPtr);
		}
		else
		{
			BucketRes = int64(NewPtr);
		}
	}

	uint32 BucketStart = GetBucketStart(BucketIndex);
	uint32 BucketOffset = Index - BucketStart;

	OutIndex = Index;

	return reinterpret_cast<T*>(BucketRes) + BucketOffset;
}

template<typename T, uint32 MinSize, uint32 MaxSize>
uint32 FPow2ChunkedArray<T, MinSize, MaxSize>::GetBucketIndex(uint32 Index) const
{
	return uint32(FPlatformMath::FloorLog2(Index / 16 + 1));
}

template<typename T, uint32 MinSize, uint32 MaxSize>
uint32 FPow2ChunkedArray<T, MinSize, MaxSize>::GetBucketStart(uint32 BucketIndex) const
{
	return BucketIndex ? 16 * ((1 << BucketIndex) - 1) : 0;
}

template<typename T, uint32 MinSize, uint32 MaxSize>
uint32 FPow2ChunkedArray<T, MinSize, MaxSize>::GetBucketSize(uint32 BucketIndex) const
{
	return FPlatformMath::Max(MinSize, 1u << (BucketIndex + SkipCount));
}

////////////////////////////////////////////////////////////////////////////////////////////////////
