// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateStreamDefinitions.h"

class IStateStreamDebugRenderer;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Interface used by TStateStream to define all the GT/RT functions required 
// to create/update/destroy the state stream on GT and to update/interpolate the result on RT

class IStateStream
{
public:
	virtual void Game_CreateLane() {}
	virtual void Game_SetLaneUserData(uint32 LaneId, void* UserData) {}
	virtual void Game_DestroyLane(uint32 LaneId) {}

	// Create current tick data that will be update later on 
	virtual void Game_BeginTick(uint32 LaneId) = 0;
	
	// Make the current tick available to the renderer (update oldest/newest ticks) and then reset it 
	virtual void Game_EndTick(StateStreamTime AbsoluteTime, uint32 LaneId) = 0;

	// Set the lane that will be used if provided lane is DefaultLaneId
	virtual void Game_SetDefaultLane(uint32 LaneId) = 0;

	// Release all the instance/ticks/dynamic states stored within that stream
	virtual void Game_Exit() = 0;
	
	// Get the state stream interface pointer
	virtual void* Game_GetVoidPointer() = 0;
	
	// Debug render streams data on GT 
	virtual void Game_DebugRender(IStateStreamDebugRenderer& Renderer) {}

	// Interpolate the instance render dynamic state from the available ticks [oldest-newest] given a time
	virtual void Render_Update(StateStreamTime AbsoluteTime) = 0;
	
	// Destroy the updated render state
	virtual void Render_PostUpdate() = 0;
	
	// Last update of the render dynamic state
	virtual void Render_Exit() = 0;
	
	// Release all the ticks/data stored before the updated render tick
	virtual void Render_GarbageCollect() = 0;
	
	// Enable the state stream render interpolation
	virtual void Render_Enable(bool Enable) = 0;

	// Get the state stream id
	virtual uint32 GetId() = 0;
	
	// Get the state stream debug name
	virtual const TCHAR* GetDebugName() { return TEXT("Unknown"); }
	
	virtual ~IStateStream() = default;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
