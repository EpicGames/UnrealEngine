// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"

#define UE_API STATESTREAM_API

class FStateStreamManagerImpl;
class IStateStream;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FStateStreamRegisterContext
{
	FStateStreamManagerImpl& Manager;

	UE_API void Register(IStateStream& StateStream, bool TakeOwnership) const;
	UE_API void RegisterDependency(uint32 FromId, uint32 ToId) const;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FStateStreamUnregisterContext
{
	FStateStreamManagerImpl& Manager;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FStateStreamCreator
{
public:
	using FRegisterFunction = TFunction<void(const FStateStreamRegisterContext&)>;
	using FUnregisterFunction = TFunction<void(const FStateStreamUnregisterContext&)>;

	UE_API FStateStreamCreator(uint32 Id, FRegisterFunction&& InRegisterFunction, FUnregisterFunction&& InUnregisterFunction);
	UE_API ~FStateStreamCreator();

	// Called by system owning StateStreamManager
	UE_API static void RegisterStateStreams(const FStateStreamRegisterContext&);
	UE_API static void UnregisterStateStreams(const FStateStreamUnregisterContext&);

private:
	FStateStreamCreator(const FStateStreamCreator&) = delete;
	FStateStreamCreator& operator=(const FStateStreamCreator&) = delete;

	FRegisterFunction RegisterFunction;
	FUnregisterFunction UnregisterFunction;

	static FStateStreamCreator* First;
	FStateStreamCreator* Next;
	FStateStreamCreator* Prev;
	uint32 Id;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#define STATESTREAM_CREATOR_INSTANCE(ImplName) \
	FStateStreamCreator ImplName##Creator(ImplName::Id, \
		[](const FStateStreamRegisterContext& Context) \
			{ ImplName& Impl = *new ImplName(); Context.Register(Impl, true); }, \
		[](const FStateStreamUnregisterContext& Context) {});

#define STATESTREAM_CREATOR_INSTANCE_WITH_DEPENDENCY(ImplName, ToId) \
	FStateStreamCreator ImplName##Creator(ImplName::Id, \
		[](const FStateStreamRegisterContext& Context) \
			{ ImplName& Impl = *new ImplName(); Context.Register(Impl, true); Context.RegisterDependency(ImplName::Id, ToId); }, \
		[](const FStateStreamUnregisterContext& Context) {});

#define STATESTREAM_CREATOR_INSTANCE_WITH_FUNC(ImplName, ...) \
	FStateStreamCreator ImplName##Creator(ImplName::Id, \
		[](const FStateStreamRegisterContext& Context) \
			{ ImplName& Impl = *new ImplName(); Context.Register(Impl, true); __VA_ARGS__(Context, Impl); }, \
		[](const FStateStreamUnregisterContext& Context) {});

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
