// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Math/Color.h"

////////////////////////////////////////////////////////////////////////////////////////////////////
// IStateStreamDebugRenderer interface to draw state streams debug data

class IStateStreamDebugRenderer
{
public:
	// Interface to draw text with a given color for debug purposes
	virtual void DrawText(const FStringView& Text, const FColor& Color) = 0;
	virtual void DrawText(uint32 X, uint32 Y, const FStringView& Text, const FColor& Color) = 0;
	virtual ~IStateStreamDebugRenderer() {}
};

////////////////////////////////////////////////////////////////////////////////////////////////////
