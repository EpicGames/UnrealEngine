// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateStreamHandle.h"


////////////////////////////////////////////////////////////////////////////////////////////////////

inline constexpr uint32 DefaultLaneId = ~0u;

////////////////////////////////////////////////////////////////////////////////////////////////////

#define DECLARE_STATESTREAM(Type) \
	using StaticState = F##Type##StaticState; \
	using DynamicState = F##Type##DynamicState; \
	using Handle = F##Type##Handle; \
	static inline constexpr uint32 Id = Type##StateStreamId; \


// TODO: A struct with multiple times (wall, game, etc)
#if !defined(UE_STATESTREAM_TIME_TYPE)
#define UE_STATESTREAM_TIME_TYPE double
#endif

using StateStreamTime = UE_STATESTREAM_TIME_TYPE;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Context used for interpolation functions

struct FStateStreamInterpolateContext : FStateStreamCopyContext
{
	double Factor = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// Generic interpolation functions. Used by code generation

template<typename Type>
inline void StateStreamInterpolate(FStateStreamInterpolateContext& Context, Type& Out, const Type& From, const Type& To)
{
	Out = To;
}


inline void StateStreamInterpolate(FStateStreamInterpolateContext& Context, uint32& Out, const uint32& From, const uint32& To)
{
	Out = From + uint32(double(To - From)*Context.Factor);
}

inline void StateStreamInterpolate(FStateStreamInterpolateContext& Context, float& Out, const float& From, const float& To)
{
	Out = From + float(double(To - From)*Context.Factor);
}

inline void StateStreamInterpolate(FStateStreamInterpolateContext& Context, double& Out, const double& From, const double& To)
{
	Out = From + (To - From)*Context.Factor;
}

template<typename Type>
inline bool StateStreamEquals(const Type& A, const Type& B)
{
	return A == B;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
