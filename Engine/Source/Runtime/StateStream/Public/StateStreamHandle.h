// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateStreamHandle.generated.h"

#define UE_API STATESTREAM_API

class IStateStreamHandleOwner;
struct FStateStreamCopyContext;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Base type for state stream handles. This code is not supposed to be used directly.
// To create your new state stream you will need a handle dedicated to that state stream. Example:
// 
//   USTRUCT(StateStreamHandle)
//   struct FFooHandle : public FStateStreamHandle
//   {
//      GENERATED_USTRUCT_BODY()
//   };
//

USTRUCT(NoExport)
struct FStateStreamHandle
{
	UE_API void MakeInternal();

	inline bool IsValid() const { return Id != 0; }
	inline uint32 GetId() const { return Id; }

	UE_API void* Render_GetUserData() const;

protected:
	FStateStreamHandle() = default;
	UE_API FStateStreamHandle(const FStateStreamHandle& Other);
	UE_API FStateStreamHandle(FStateStreamHandle&& Other);
	UE_API FStateStreamHandle& operator=(FStateStreamHandle&& Other);
	UE_API FStateStreamHandle& operator=(const FStateStreamHandle& Other);
	UE_API ~FStateStreamHandle();

	UE_API FStateStreamHandle(IStateStreamHandleOwner& O, uint32 I);
	UE_API FStateStreamHandle(FStateStreamCopyContext& Context, const FStateStreamHandle& Other);
	UE_API void Apply(FStateStreamCopyContext& Context, const FStateStreamHandle& Other);

	UE_API bool operator==(const FStateStreamHandle& Other) const;

	IStateStreamHandleOwner* Owner = nullptr;
	uint32 IsInternal = 0;
	uint32 Id = 0;
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// StateStream handle owner.

class IStateStreamHandleOwner
{
public:
	virtual void Game_AddRef(uint32 HandleId) = 0;
	virtual void Game_Release(uint32 HandleId) = 0;
	virtual void Game_Update(uint32 HandleId, const void* DynamicState, double TimeFactor, uint64 UserData) = 0;
	virtual void* Game_Edit(uint32 HandleId, double TimeFactor, uint64 UserData) = 0;
	virtual void* Render_GetUserData(uint32 HandleId) = 0;
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Copy context used to copy state stream handles.

struct FStateStreamCopyContext
{
	uint32 IsInternal = 1;
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Macro added by UHT to USTRUCT(StateStreamHandle) types

#define DECLARE_STATESTREAM_HANDLE(Type) \
	F##Type##Handle() = default; \
	F##Type##Handle(const F##Type##Handle& Other) : FStateStreamHandle(Other) {} \
	F##Type##Handle(F##Type##Handle&& Other) : FStateStreamHandle(std::move(Other)) {} \
	F##Type##Handle& operator=(F##Type##Handle&& Other) { FStateStreamHandle::operator=(std::move(Other)); return *this; } \
	F##Type##Handle& operator=(const F##Type##Handle& Other) { FStateStreamHandle::operator=(Other); return *this; } \
	~F##Type##Handle() = default; \
	void Update(const F##Type##DynamicState& Ds, double TimeFactor = 0) { if (Owner) Owner->Game_Update(Id, &Ds, TimeFactor, 0); } \
	F##Type##DynamicState& Edit(double TimeFactor = 0) { return *static_cast<F##Type##DynamicState*>(Owner->Game_Edit(Id, TimeFactor, 0)); } \
	F##Type##Handle(IStateStreamHandleOwner& O, uint32 I) : FStateStreamHandle(O, I) {} \
	F##Type##Handle(FStateStreamCopyContext& C, const F##Type##Handle& O) : FStateStreamHandle(C, O) {} \
	void Apply(FStateStreamCopyContext& Context, const FStateStreamHandle& Other) { FStateStreamHandle::Apply(Context, Other); } \
	bool operator==(const F##Type##Handle& Other) const { return FStateStreamHandle::operator==(Other); } \
	static inline constexpr const TCHAR* DebugName = TEXT(#Type); \
	using FStateStreamHandle::Apply; \
	friend F##Type##DynamicState; \

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
