// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class IStateStreamDebugRenderer;

////////////////////////////////////////////////////////////////////////////////////////////////////
// StateStreamManager interface. This should be used from Game side

class IStateStreamManager
{
public:
	virtual uint32 Game_CreateLane() { return 0; }
	virtual void Game_SetLaneUserData(uint32 LaneId, void* UserData) {}
	virtual void Game_DestroyLane(uint32 LaneId) {}

	// Call from Game when a new tick is opened
	// Note, no state stream handles can be created,updated,destroyed outside a Begin/End tick
	virtual void Game_BeginTick(uint32 LaneId) = 0;

	// Close tick and make it available to render side.
	// AbsoluteTime is the amount of time that Game consumed
	virtual void Game_EndTick(double AbsoluteTime, uint32 LaneId) = 0;

	// Should be called when game is exiting.
	virtual void Game_Exit() = 0;

	// Returns true if game is inside an open tick
	virtual bool Game_IsInTick(uint32 LaneId) = 0;

	// Set the lane that will be used if provided lane is DefaultLaneId. Returns previous default lane id
	virtual uint32 Game_SetDefaultLane(uint32 LaneId) = 0;

	// Functions to fetch StateStream interface (not IStateStream) game side.
	template<typename T>
	T& Game_Get() { return *static_cast<T*>(Game_GetStreamPointer(T::Id)); }
	virtual void* Game_GetStreamPointer(uint32 Id) = 0;

	// StateStream debug rendering
	virtual void Game_DebugRender(IStateStreamDebugRenderer& Renderer, uint32 LaneId) = 0;

protected:
	virtual ~IStateStreamManager() = default;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FStateStreamLaneScope
{
	FStateStreamLaneScope(IStateStreamManager* InManager, uint32 LaneId) : Manager(InManager)
	{
		if (Manager)
		{
			OldLaneId = Manager->Game_SetDefaultLane(LaneId);
		}
	}
	~FStateStreamLaneScope()
	{
		if (Manager)
		{
			Manager->Game_SetDefaultLane(OldLaneId);
		}
	}

	IStateStreamManager* Manager;
	uint32 OldLaneId;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
