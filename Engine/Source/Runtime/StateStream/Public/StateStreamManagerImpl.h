// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateStream.h"
#include "StateStreamManager.h"
#include "Tasks/Task.h"

#define UE_API STATESTREAM_API

////////////////////////////////////////////////////////////////////////////////////////////////////
// StateStreamManager implementation. 
// This type should only be known render side.

class FStateStreamManagerImpl : public IStateStreamManager
{
public:
	//~ Begin IStateStreamManager interface
	UE_API virtual uint32 Game_CreateLane();
	UE_API virtual void Game_SetLaneUserData(uint32 LaneId, void* UserData);
	UE_API virtual void Game_DestroyLane(uint32 LaneId);
	UE_API virtual void Game_BeginTick(uint32 LaneId) override;
	UE_API virtual void Game_EndTick(double AbsoluteTime, uint32 LaneId) override;
	UE_API virtual void Game_Exit() override;
	UE_API virtual bool Game_IsInTick(uint32 LaneId) override;
	UE_API virtual uint32 Game_SetDefaultLane(uint32 LaneId) override;
	UE_API virtual void* Game_GetStreamPointer(uint32 Id) override;
	UE_API virtual void Game_DebugRender(IStateStreamDebugRenderer& Renderer, uint32 LaneId) override;
	//~ End IStateStreamManager interface

	// Register new state streams into manager.
	// TakeOwnership true means that manager will delete stream when shutting down
	UE_API void Render_Register(IStateStream& Stream, bool TakeOwnership);

	// Register dependency between state streams given their ids. FromId will depend on ToId
	UE_API void Render_RegisterDependency(uint32 FromId, uint32 ToId);
	
	// Register dependency between state streams given their interfaces. From will depend on To
	UE_API void Render_RegisterDependency(IStateStream& From, IStateStream& To);

	// Called at the beginning of a render frame. AbsolutTime is the amount of time the render frame consumes
	UE_API void Render_Update(double AbsoluteTime);

	// Called before Render thread exits
	UE_API void Render_Exit();

	// Garbage collect all the unnecessary ticks/data within each streams
	UE_API void Render_GarbageCollect(bool AsTask = false);

	// Get state stream from id
	UE_API IStateStream* Render_GetStream(uint32 Id);
	
	UE_API virtual ~FStateStreamManagerImpl() override;
	
private:
	
	// Small structure used to list the state streams available in the manager 
	struct StateStreamRec
	{
		IStateStream* Stream;
		bool Owned;
	};
	
	// List of state streams available in the manager 
	TArray<StateStreamRec> StateStreams;
	
	// List of state streams for each registered state stream id. 
	// This will allow fast lookup to retrieve a state stream given an id
	TArray<IStateStream*> StateStreamsLookup;

	uint32 ActiveDefaultLaneId = DefaultLaneId;

	// Bool to check if we are currently in between the begin and the end tick
	bool bIsInTick = false;
	
	// Bool to check if the game has already been exited on the GT side
	bool bGameExited = false;
	
	// Bool to check if the game has already been exited on the RT side
	bool bRenderExited = false;

	// Async task to perform the garbage collection
	UE::Tasks::FTask GarbageCollectTask;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
