// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "Pow2ChunkedArray.h"

////////////////////////////////////////////////////////////////////////////////////////////////////
// TStateStreamStore is a threadsafe, reference stable storage. User can hold on to pointers of
// elements while other elements are added/removed.
// Store is backed by a Pow2ChunkedArray and use a "free list" to be able to reuse removed elements

template<typename T>
class TStateStreamStore
{
public:
	inline TStateStreamStore();
	inline ~TStateStreamStore();

	inline uint32 Add(const T& Value);
	inline void* AddUninitialized(uint32& OutIndex);
	inline void Remove(uint32 Index);

	template <typename... ArgsType>
	inline uint32 Emplace(ArgsType&&... Args);

	inline T& operator[](uint32 Index);

	inline uint32 GetUsedCount() const;

private:
	FRWLock Lock;
	FPow2ChunkedArray<T> Array;
	uint32 FirstFreeIndex = ~0u;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// Implementation

template<typename T>
TStateStreamStore<T>::TStateStreamStore()
{
	static_assert(sizeof(T) >= sizeof(uint32));
}

template<typename T>
TStateStreamStore<T>::~TStateStreamStore()
{
	if (!std::is_trivially_destructible_v<T>)
	{
		while (FirstFreeIndex != ~0u)
		{
			T& Element = Array[FirstFreeIndex];
			FirstFreeIndex = *reinterpret_cast<uint32*>(&Element);
			new (&Element) T();
		}
	}
}

template<typename T>
uint32 TStateStreamStore<T>::Add(const T& Value)
{
	uint32 Index;
	new (AddUninitialized(Index)) T(Value);
	return Index;
}

template<typename T>
void* TStateStreamStore<T>::AddUninitialized(uint32& OutIndex)
{
	Lock.WriteLock();
	if (FirstFreeIndex == ~0u)
	{
		Lock.WriteUnlock();
		return Array.AddUninitialized(OutIndex);
	}

	uint32 Index = FirstFreeIndex;
	void* Element = &Array[Index];
	FirstFreeIndex = *reinterpret_cast<uint32*>(Element);
	Lock.WriteUnlock();

	OutIndex = Index;
	return &Array[Index];
}

template<typename T>
void TStateStreamStore<T>::Remove(uint32 Index)
{
	T& Element = Array[Index];
	Element.~T();
	Lock.WriteLock();
	*reinterpret_cast<uint32*>(&Element) = FirstFreeIndex;
	FirstFreeIndex = Index;
	Lock.WriteUnlock();
}

template<typename T>
template <typename... ArgsType>
uint32 TStateStreamStore<T>::Emplace(ArgsType&&... Args)
{
	uint32 OutIndex;
	new (AddUninitialized(OutIndex)) T(Forward<ArgsType>(Args)...);
	return OutIndex;
}

template<typename T>
T& TStateStreamStore<T>::operator[](uint32 Index)
{
	return Array[Index];
}

template<typename T>
uint32 TStateStreamStore<T>::GetUsedCount() const
{
	uint32 UsedCount = Array.Num();
	uint32 Index = FirstFreeIndex;
	while (Index != ~0u)
	{
		--UsedCount;
		Index = *(uint32*)&Array[Index];
	}
	return UsedCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
