// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TransformStateStreamHandle.h"
#include "TransformStateStreamMath.h"
#include "TransformStateStream.generated.h"


////////////////////////////////////////////////////////////////////////////////////////////////////
// Static state for transform instance. Can only be set upon creation

USTRUCT(StateStreamStaticState)
struct FTransformStaticState
{
	GENERATED_USTRUCT_BODY()
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Dynamic state for transform instance. Can be updated inside ticks

USTRUCT(StateStreamDynamicState)
struct FTransformDynamicState
{
	GENERATED_USTRUCT_BODY()

private: // Use accessors (SetX/GetX) instead of properties directly

	// Transform relative Parent
	UPROPERTY()
	FTransform LocalTransform = FTransform::Identity;

	UPROPERTY()
	TArray<FTransform> BoneTransforms;

	// Parent
	UPROPERTY()
	FTransformHandle Parent;

	//UPROPERTY()
	//uint8 LocationRule = 0;

	//UPROPERTY()
	//uint8 RotationRule = 0;

	//UPROPERTY()
	//uint8 ScaleRule = 0;

	UPROPERTY()
	bool bVisible = true;
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Transform state stream id used for registering dependencies and find statestream

inline constexpr uint32 TransformStateStreamId = 1;


////////////////////////////////////////////////////////////////////////////////////////////////////
// Interface for creating transform instances

class ITransformStateStream
{
public:
	DECLARE_STATESTREAM(Transform)
	virtual FTransformHandle Game_CreateInstance(const FTransformStaticState& Ss, const FTransformDynamicState& Ds, uint32 LaneId = DefaultLaneId) = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
