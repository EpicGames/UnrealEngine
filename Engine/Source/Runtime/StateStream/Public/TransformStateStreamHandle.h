// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateStreamHandle.h"
#include "TransformStateStreamHandle.generated.h"

struct FTransformDynamicState;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Handle for transform instance

USTRUCT(StateStreamHandle)
struct FTransformHandle : public FStateStreamHandle
{
	GENERATED_USTRUCT_BODY()
};

////////////////////////////////////////////////////////////////////////////////////////////////////
