// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericStateStream.h"
#include "Templates/RefCounting.h"
#include "TransformStateStream.h"

#define UE_API STATESTREAM_API

////////////////////////////////////////////////////////////////////////////////////////////////////
// Listener for when transform object gets dirty.
// Inherit FTransformObjectListener and register to FTransformObject

class FTransformObjectListener
{
public:
	virtual void OnTransformObjectDirty() = 0;

private:
	FTransformObjectListener* Prev = nullptr;
	FTransformObjectListener* Next= nullptr;
	friend class FTransformObject;
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Transform object representing a instantiated transform.

class FTransformObject : public TRefCountingMixin<FTransformObject>, public FTransformObjectListener
{
public:
	struct Info
	{
		const FTransform& WorldTransform;
		const TArray<FTransform>& BoneTransforms;
		bool bVisible;
	};

	UE_API Info GetInfo();

	UE_API void AddListener(FTransformObjectListener* Listener);
	UE_API void RemoveListener(FTransformObjectListener* Listener);

private:
	virtual ~FTransformObject();
	void CallListeners();
	virtual void OnTransformObjectDirty() override final;

	TRefCountPtr<FTransformObject> Parent;
	FTransformObjectListener* First = 0;
	const FTransformDynamicState* LocalState = nullptr;
	FTransform WorldTransform;
	bool bDirty = true;
	bool bVisible = true;

	friend class FTransformStateStreamImpl;
	friend class TRefCountingMixin<FTransformObject>;
};


////////////////////////////////////////////////////////////////////////////////////////////////////

class FTransformStateStreamImpl : public TStateStream<TStateStreamSettings<ITransformStateStream, FTransformObject>>
{
private:
	void SetParent(FTransformObject* Object, const FTransformDynamicState& Ds);
	UE_API virtual void Render_OnCreate(const FTransformStaticState& Ss, const FTransformDynamicState& Ds, FTransformObject*& UserData, bool IsDestroyedInSameFrame) override;
	UE_API virtual void Render_OnUpdate(const FTransformStaticState& Ss, const FTransformDynamicState& Ds, FTransformObject*& UserData) override;
	UE_API virtual void Render_OnDestroy(const FTransformStaticState& Ss, const FTransformDynamicState& Ds, FTransformObject*& UserData) override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
