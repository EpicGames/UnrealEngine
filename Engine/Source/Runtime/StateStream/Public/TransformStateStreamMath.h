// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Transform.h"
#include "StateStreamDefinitions.h"

////////////////////////////////////////////////////////////////////////////////////////////////////
// Special implementations for types used in state stream states
// Used by generated code

inline void StateStreamInterpolate(FStateStreamInterpolateContext& Context, FTransform& Out, const FTransform& From, const FTransform& To)
{
	Out.Blend(From, To, Context.Factor);
}

inline bool StateStreamEquals(const FTransform& A, const FTransform& B)
{
	return false;
}

inline void StateStreamInterpolate(FStateStreamInterpolateContext& Context, TArray<FTransform>& Out, const TArray<FTransform>& From, const TArray<FTransform>& To)
{
	int32 Num = To.Num();
	check(Num == From.Num());
	Out.SetNum(Num);
	for (int32 I=0; I!=Num; ++I)
	{
		Out[I].Blend(From[I], To[I], Context.Factor);
	}
}

inline bool StateStreamEquals(const TArray<FTransform>& A, const TArray<FTransform>& B)
{
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
