// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StateStream : ModuleRules
{
	public StateStream(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
				"CoreUObject"
            }
        );
    }
}
